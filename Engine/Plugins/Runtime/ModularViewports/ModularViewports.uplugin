{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Modular Viewports",
	"Description": "Viewport Client implementations that provide more flexibility and granularity than Game Viewport Client.",
	"Category": "Rendering",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules":
	[
		{
			"Name": "ModularViewports",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	]
}
