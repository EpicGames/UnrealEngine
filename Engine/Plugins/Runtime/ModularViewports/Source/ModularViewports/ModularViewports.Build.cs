// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ModularViewports : ModuleRules
	{
		public ModularViewports(ReadOnlyTargetRules Target) : base(Target)
		{
			bAllowUETypesInNamespaces = true;

			PublicDependencyModuleNames.AddRange([
				"Core",
				"CoreUObject",
				"DeveloperSettings",
				"Engine",
				"UMG",
			]);

			PrivateDependencyModuleNames.AddRange([
				"ApplicationCore",
				"EngineSettings",
				"InputCore",
				"RenderCore",
				"Renderer",
				"RHI",
				"Slate",
				"SlateCore",
			]);

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.Add("UnrealEd");
			}
		}
	}
}
