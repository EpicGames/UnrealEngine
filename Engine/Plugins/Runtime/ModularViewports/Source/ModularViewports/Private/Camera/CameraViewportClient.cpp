// Copyright Epic Games, Inc. All Rights Reserved.

#include "Camera/CameraViewportClient.h"
#include "ViewportFunctions.h"
#include "CanvasTypes.h"
#include "CollisionQueryParams.h"
#include "UnrealClient.h"
#include "SceneView.h"
#include "Engine/Engine.h"
#include "Engine/HitResult.h"
#include "Engine/LocalPlayer.h"
#include "Engine/World.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/WorldSettings.h"
#include "Camera/CameraComponent.h"
#include "Components/PrimitiveComponent.h"
#include "SceneManagement.h"

namespace UE::CameraViewportClient
{
	using namespace UE::Engine;

	namespace
	{
		void CalculateProjectionData(const FViewport& Viewport, const UCameraComponent& Camera, FSceneViewProjectionData& OutProjectionData)
		{
			const FTransform CameraTransform = Camera.GetComponentTransform();
			OutProjectionData.ViewOrigin = CameraTransform.GetLocation();

			FIntPoint ViewportSize = Viewport.GetSizeXY();
			OutProjectionData.SetViewRectangle(FIntRect(0, 0, ViewportSize.X, ViewportSize.Y));

			OutProjectionData.ViewRotationMatrix = FInverseRotationMatrix(CameraTransform.Rotator()) * FMatrix(
				FPlane(0, 0, 1, 0),
				FPlane(1, 0, 0, 0),
				FPlane(0, 1, 0, 0),
				FPlane(0, 0, 0, 1));
			const float MinZ = GNearClippingPlane;
			const float MaxZ = MinZ;

			float XAxisMultiplier;
			float YAxisMultiplier;
			const EAspectRatioAxisConstraint FovType = Camera.bOverrideAspectRatioAxisConstraint
				? Camera.AspectRatioAxisConstraint
				: GetDefault<ULocalPlayer>()->AspectRatioAxisConstraint;
			const bool bMaintainXFOV = (ViewportSize.X > ViewportSize.Y && FovType == AspectRatio_MajorAxisFOV) || FovType == AspectRatio_MaintainXFOV;
			const float ViewportAspectRatio = FMath::Max(1.0f, static_cast<float>(ViewportSize.X)) / FMath::Max(1.0f, static_cast<float>(ViewportSize.Y));
			float MatrixFov;
			if (bMaintainXFOV)
			{
				XAxisMultiplier = 1.0f;
				YAxisMultiplier = ViewportAspectRatio;
				MatrixFov = FMath::Max(0.001f, Camera.FieldOfView) * UE_PI / 360.0f;
			}
			else
			{
				XAxisMultiplier = 1 / ViewportAspectRatio;
				YAxisMultiplier = 1.0f;
				const float HalfXFov = FMath::DegreesToRadians(FMath::Max(0.001f, Camera.FieldOfView) / 2.f);
				const float HalfYFov = FMath::Atan(FMath::Tan(HalfXFov) / FMath::Max(0.001f, Camera.AspectRatio));
				MatrixFov = HalfYFov;
			}
			OutProjectionData.ProjectionMatrix = FReversedZPerspectiveMatrix(MatrixFov, MatrixFov, XAxisMultiplier, YAxisMultiplier, MinZ, MaxZ);
		}

		TUniquePtr<FSceneViewFamilyContext> BuildViewFamily(const FViewport& Viewport, FSceneViewStateReference& ViewState, const UWorld& World, const UCameraComponent& Camera)
		{
			TUniquePtr<FSceneViewFamilyContext> ViewFamily = MakeGameplayViewFamily(Viewport, World);

			FSceneViewInitOptions ViewInitOptions;

			const AWorldSettings* const WorldSettings = World.GetWorldSettings();
			if (WorldSettings)
			{
				ViewInitOptions.WorldToMetersScale = WorldSettings->WorldToMeters;
			}

			CalculateProjectionData(Viewport, Camera, ViewInitOptions);

			ViewInitOptions.ViewFamily = ViewFamily.Get();
			ViewInitOptions.SceneViewStateInterface = ViewState.GetReference();

			ViewInitOptions.BackgroundColor = FLinearColor::Black;
			ViewInitOptions.FOV = Camera.FieldOfView;

			const FTransform CameraTransform = Camera.GetComponentTransform();
			FSceneView* View = new FSceneView(ViewInitOptions);
			View->ViewLocation = CameraTransform.GetLocation();
			View->ViewRotation = CameraTransform.Rotator();
			ViewFamily->Views.Add(View);

			View->StartFinalPostprocessSettings(View->ViewLocation);
			View->OverridePostProcessSettings(Camera.PostProcessSettings, Camera.PostProcessBlendWeight);
			View->EndFinalPostprocessSettings(ViewInitOptions);

			return ViewFamily;
		}
	}
}

namespace UE
{
	FCameraViewportClient::FCameraViewportClient(
		const UCameraComponent& InCameraComponent,
		ETouchEventFamilies InEnabledTouchEventFamilies,
		ECollisionChannel InTouchTraceChannel,
		float InTouchTraceDistance)
		: Camera(&InCameraComponent)
		, EnabledTouchEventFamilies(InEnabledTouchEventFamilies)
		, TouchTraceChannel(InTouchTraceChannel)
		, TouchTraceDistance(InTouchTraceDistance)
	{
	}

	UWorld* FCameraViewportClient::GetWorld() const
	{
		return Camera.IsValid()
			? Camera->GetWorld()
			: nullptr;
	}

	void FCameraViewportClient::AddReferencedObjects(FReferenceCollector& Collector)
	{
		FSceneViewStateInterface* Ref = ViewState.GetReference();
		if (Ref)
		{
			Ref->AddReferencedObjects(Collector);
		}
	}

	FString FCameraViewportClient::GetReferencerName() const
	{
		return TEXT("FCameraViewportClient");
	}

	void FCameraViewportClient::CalculateProjectionData(const FViewport& Viewport, FSceneViewProjectionData& OutProjectionData) const
	{
		if (!Camera.IsValid())
		{
			return;
		}

		CameraViewportClient::CalculateProjectionData(Viewport, *Camera, OutProjectionData);
	}

	void FCameraViewportClient::Draw(FViewport* Viewport, FCanvas* Canvas)
	{
		using namespace CameraViewportClient;
		using namespace UE::Engine;

		check(Viewport);
		check(Canvas);
		UWorld* World = GetWorld();
		if (!World)
		{
			return;
		}

		// Hypothetically, a camera can move from one world to another
		if (ViewStateWorld != World)
		{
			ViewState.Destroy();
			ViewState.Allocate(World->GetFeatureLevel());
			ViewStateWorld = World;
		}

		TUniquePtr<FSceneViewFamilyContext> ViewFamily = BuildViewFamily(*Viewport, ViewState, *World, *Camera);
		AddStreamingViewInfo(*World, *ViewFamily->Views[0]);
		TryRenderViewFamily(*ViewFamily, *Canvas);
	}

	bool FCameraViewportClient::InputTouch(FViewport* const Viewport, const FTouchId TouchId, const ETouchType::Type Type,
		const FVector2D& TouchLocation, const float Force, const uint64 Timestamp)
	{
		UWorld* const World = GetWorld();
		if (!World || !Viewport || !Camera.IsValid())
		{
			return false;
		}

		FSceneViewProjectionData ProjectionData;
		CalculateProjectionData(*Viewport, ProjectionData);

		FHitResult HitResult;
		const FCollisionQueryParams QueryParams(SCENE_QUERY_STAT(CameraViewportTouch), /*bTraceComplex=*/ true);
		const bool bHit = World->LineTraceFromScreenProjection(HitResult, ProjectionData, TouchLocation, TouchTraceChannel, QueryParams, TouchTraceDistance);

		UPrimitiveComponent::DispatchTouchHitResult(
			TouchId, Type, HitResult, bHit, /*Controller=*/ nullptr,
			EnabledTouchEventFamilies,
			TouchedPrimitives);

		return true;
	}
}
