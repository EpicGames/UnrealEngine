// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/SceneViewportWidget.h"
#include "Camera/CameraComponent.h"
#include "Engine/LocalPlayer.h"
#include "Engine/AuxiliaryGameInstance.h"
#include "Camera/CameraViewportClient.h"
#include "GameFramework/PlayerViewportClient.h"
#include "ViewportFunctions.h"
#include "Slate/SceneViewport.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SViewport.h"
#include "Slate/SGameLayerManager.h"
#include "Engine/Engine.h"
#include "Engine/GameInstance.h"
#include "Engine/GameViewportClient.h"

#define LOCTEXT_NAMESPACE "UMG"

USceneViewport::USceneViewport(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bRenderDirectlyToWindow(false)
	, bEnableGammaCorrection(true)
	, bReverseGammaCorrection(false)
	, bEnableBlending(false)
	, bPreMultipliedAlpha(true)
	, bIgnoreTextureAlpha(true)
{
	bIsVariable = true;
}

UGameInstance* USceneViewport::GetInnerGameInstance() const
{
	return GameInstanceAssembly.IsValid() ? GameInstanceAssembly->GetGameInstance() : nullptr;
}

void USceneViewport::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	if (GEngine && SceneViewport.IsValid())
	{
		GEngine->UnregisterViewport(SceneViewport.ToSharedRef());
	}
	ViewportWidget.Reset();
}

void USceneViewport::BeginDestroy()
{
	Unbind();
	GameInstanceAssembly.Reset();
	Client.Reset();
	Super::BeginDestroy();
}

TSharedRef<SWidget> USceneViewport::RebuildWidget()
{
	using namespace UE::Engine;

	if (IsDesignTime())
	{
		return SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
					.Text(LOCTEXT("Viewport", "Viewport"))
			];
	}

	if (!Client.IsValid() && !AutoLoadWorldAsset.IsNull())
	{
		if (!GameInstanceAssembly.IsValid())
		{
			GameInstanceAssembly = FAuxiliaryGameInstance::MakeUnique(AutoLoadWorldAsset);
		}
		if (UGameInstance* GameInstance = GameInstanceAssembly->GetGameInstance())
		{
			Client = GameInstance->GetGameViewportClient();
			Bind();
		}
	}

	ViewportWidget = SNew(SViewport)
		.ShowEffectWhenDisabled(false) // Editor-specific behavior
		.RenderDirectlyToWindow(bRenderDirectlyToWindow)
		.EnableGammaCorrection(bEnableGammaCorrection)
		.ReverseGammaCorrection(bReverseGammaCorrection)
		.EnableBlending(bEnableBlending)
		.PreMultipliedAlpha(bPreMultipliedAlpha)
		.IgnoreTextureAlpha(bIgnoreTextureAlpha);

	Bind();

	return ViewportWidget.ToSharedRef();
}

#if WITH_EDITOR

const FText USceneViewport::GetPaletteCategory()
{
	return LOCTEXT("Primitive", "Primitive");
}

#endif

void USceneViewport::ShowGame(const TSoftObjectPtr<UWorld>& Asset)
{
	using namespace UE::Engine;

	TUniquePtr<FAuxiliaryGameInstance> NewAssembly = FAuxiliaryGameInstance::MakeUnique(Asset);
	UGameInstance* GameInstance = NewAssembly->GetGameInstance();
	if (!GameInstance)
	{
		return;
	}

	GameInstanceAssembly = MoveTemp(NewAssembly);
	Client = GameInstance->GetGameViewportClient();
	Bind();
}

void USceneViewport::ShowCamera(const UCameraComponent& Camera)
{
	Client = MakeShared<UE::FCameraViewportClient>(Camera);
	Bind();
}

void USceneViewport::ShowPlayer(ULocalPlayer& Player)
{
	using namespace UE::Engine;

	UPlayerViewportClient* const PlayerClient = UPlayerViewportClient::Create(Player, this);
	Client = PlayerClient;
	Bind();
}

void USceneViewport::Unbind()
{
	if (GEngine && SceneViewport.IsValid())
	{
		GEngine->UnregisterViewport(SceneViewport.ToSharedRef());
	}
	SceneViewport.Reset();
}

void USceneViewport::Bind()
{
	Unbind();

	if (!ViewportWidget.IsValid() || !Client.IsValid())
	{
		return;
	}

	SceneViewport = FSceneViewport::Create(Client, ViewportWidget);
	if (LIKELY(GEngine))
	{
		GEngine->RegisterViewport(SceneViewport.ToSharedRef());
	}
}

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
