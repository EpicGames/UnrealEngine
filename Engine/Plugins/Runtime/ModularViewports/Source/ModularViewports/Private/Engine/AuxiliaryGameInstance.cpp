// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/AuxiliaryGameInstance.h"

#include "GameMapsSettings.h"
#include "Engine/AuxiliaryGameViewportClient.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "Engine/GameInstance.h"
#include "Engine/LocalPlayer.h"
#include "GameFramework/Pawn.h"
#include "ModularViewportsSettings.h"
#if WITH_EDITOR
#include "Editor.h"
#include "Editor/EditorEngine.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(LogAuxiliaryGame, Log, All);

namespace UE::Engine
{

TUniquePtr<FAuxiliaryGameInstance> FAuxiliaryGameInstance::MakeUnique(const TSoftObjectPtr<UWorld>& Asset)
{
	return TUniquePtr<FAuxiliaryGameInstance>(Make(Asset));
}

TSharedPtr<FAuxiliaryGameInstance> FAuxiliaryGameInstance::MakeShared(const TSoftObjectPtr<UWorld>& Asset)
{
	return MakeShareable(Make(Asset));
}

FAuxiliaryGameInstance* FAuxiliaryGameInstance::Make(const TSoftObjectPtr<UWorld>& Asset)
{
	check(IsInGameThread());
	if (!GEngine)
	{
		UE_LOGF(LogAuxiliaryGame, Error, "Cannot create additional world before the engine is initialized.")
		return nullptr;
	}

#if WITH_EDITOR
	if (GEngine->IsEditor())
	{
		UEditorEngine* EditorEngine = CastChecked<UEditorEngine>(GEngine);
		if (!EditorEngine->IsPlayingSessionInEditor())
		{
			UE_LOGF(LogAuxiliaryGame, Error, "Cannot load additional world outside of PIE.");
			return nullptr;
		}
	}
#endif

	const FSoftClassPath GameInstanceClassName = GetDefault<UGameMapsSettings>()->GameInstanceClass;
	const UClass* GameInstanceClass = GameInstanceClassName.TryLoadClass<UGameInstance>();
	if (!GameInstanceClass)
	{
		GameInstanceClass = UGameInstance::StaticClass();
	}
	const TStrongObjectPtr<UGameInstance> GameInstance = TStrongObjectPtr(NewObject<UGameInstance>(GEngine, GameInstanceClass));
	if (!GameInstance)
	{
		UE_LOGF(LogAuxiliaryGame, Error, "Failed to create GameInstance");
		return nullptr;
	}

	GameInstance->InitializeStandalone();

#if WITH_EDITOR
	if (GEngine->IsEditor())
	{
		// "Inherit" PIE behavior from the PIE code that spawned me.
		FWorldContext* Context = GameInstance->GetWorldContext();
		Context->WorldType = EWorldType::PIE;
		// "PIE Instance" (AKA "Index") = nonce to prevent my (ephemeral) object paths from being the same as those in the (canonical) `.umap` from
		// which they are cloned/instantiated.  Only editor-managed game instances care about being contiguous.  We need to avoid having the same
		// index as any other world including editor-managed ones.
		static int32 NextAdditionalPIEIndex = -2;
		if (++NextAdditionalPIEIndex < 0)
		{
			// Overflowed, loop around (or first init)
			NextAdditionalPIEIndex = MAX_PIE_INSTANCES + 1;
		}
		Context->PIEInstance = NextAdditionalPIEIndex;
	}
#endif

	{
		FString Error;
		FURL URL(nullptr, *Asset.GetLongPackageName(), TRAVEL_Absolute);
		if (!GEngine->LoadMap(*GameInstance->GetWorldContext(), URL, nullptr, Error))
		{
			UE_LOGF(LogAuxiliaryGame, Error, "Failed to browse to world: %ls", *Error);
		}
	}

	FAuxiliaryGameInstance* Instance = new FAuxiliaryGameInstance(*GameInstance);
	return Instance;
}

FAuxiliaryGameInstance::FAuxiliaryGameInstance(UGameInstance& InGameInstance)
	: GameInstance(&InGameInstance)
{
#if WITH_EDITOR
	if (GEditor)
	{
		EndPIEHandle = FEditorDelegates::EndPIE.AddLambda([this](const bool /*bIsSimulating*/)
		{
			Teardown();
		});
	}
#endif

	// Client
	UClass* ViewportClientClass = GetDefault<UModularViewportsSettings>()->AuxiliaryGameInstanceClass;
	if (!ViewportClientClass)
	{
		ViewportClientClass = UAuxiliaryGameViewportClient::StaticClass();
	}
	UGameViewportClient& Client = *NewObject<UGameViewportClient>(GEngine, ViewportClientClass);
	Client.Init(*InGameInstance.GetWorldContext(), &InGameInstance);

#if WITH_EDITOR
	if (GEngine->IsEditor())
	{
		Client.bIsPlayInEditorViewport = true;
	}
#endif
	InGameInstance.GetWorldContext()->GameViewport = &Client;

	FString Error;
	ULocalPlayer* LocalPlayer = Client.SetupInitialLocalPlayer(Error);
	if (!LocalPlayer)
	{
		UE_LOGF(LogAuxiliaryGame, Error, "Failed to create LocalPlayer for world: %ls", *Error);
	}
}

FAuxiliaryGameInstance::~FAuxiliaryGameInstance()
{
#if WITH_EDITOR
	if (EndPIEHandle.IsValid() && GEditor)
	{
		FEditorDelegates::EndPIE.Remove(EndPIEHandle);
	}
	EndPIEHandle.Reset();
#endif
	Teardown();
}

void FAuxiliaryGameInstance::Teardown()
{
	if (!GameInstance.IsValid())
	{
		return;
	}

	UWorld* World = GameInstance->GetWorld();

	// If the world is already torn down (e.g. EndPlayMap already cleaned up this PIE context), skip manual cleanup to avoid double-destroy.
	if (!World || World->bIsTearingDown)
	{
		GameInstance.Reset();
		return;
	}

	if (GameInstance->GetWorldContext() && GameInstance->GetWorldContext()->OwningGameInstance)
	{
		for (auto It = GameInstance->GetLocalPlayerIterator(); It; ++It)
		{
			ULocalPlayer* Player = *It;
			if (Player->PlayerController && Player->PlayerController->GetWorld() == World)
			{
				if (Player->PlayerController->GetPawn())
				{
					World->DestroyActor(Player->PlayerController->GetPawn(), true);
				}
				World->DestroyActor(Player->PlayerController, true);
				Player->PlayerController = nullptr;
			}
			Player->CleanupViewState();
		}
	}

	World->BeginTearingDown();
	World->EndPlay(EEndPlayReason::RemovedFromWorld);
	World->CleanupWorld();
	World->RemoveFromRoot();
	GameInstance->Shutdown();

	if (GEngine)
	{
		GEngine->DestroyWorldContext(World);
	}

	GameInstance.Reset();
}

} // namespace UE::Engine
