// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/AuxiliaryGameInstanceObject.h"

#include "UObject/Package.h"

UAuxiliaryGameInstance* UAuxiliaryGameInstance::Make(const TSoftObjectPtr<UWorld>& Asset)
{
	// MakeUnique calls GEngine->LoadMap, which flushes async loading and can run GC. Construct the inner instance
	// first so no unreferenced UAuxiliaryGameInstance exists during that call.
	TUniquePtr<UE::Engine::FAuxiliaryGameInstance> Inner = UE::Engine::FAuxiliaryGameInstance::MakeUnique(Asset);
	if (!Inner)
	{
		return nullptr;
	}
	UAuxiliaryGameInstance* Wrapper = NewObject<UAuxiliaryGameInstance>(GetTransientPackage());
	Wrapper->Inner = MoveTemp(Inner);
	return Wrapper;
}

UGameInstance* UAuxiliaryGameInstance::GetGameInstance() const
{
	return Inner ? Inner->GetGameInstance() : nullptr;
}

UWorld* UAuxiliaryGameInstance::GetWorld() const
{
	return Inner ? Inner->GetWorld() : nullptr;
}

void UAuxiliaryGameInstance::BeginDestroy()
{
	Inner.Reset();
	Super::BeginDestroy();
}
