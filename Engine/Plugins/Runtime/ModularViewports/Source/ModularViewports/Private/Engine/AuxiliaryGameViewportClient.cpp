// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/AuxiliaryGameViewportClient.h"
#include "Engine/Engine.h"
#include "Engine/GameInstance.h"
#include "Engine/LocalPlayer.h"
#include "Engine/World.h"

DEFINE_LOG_CATEGORY_STATIC(LogAuxiliaryGameViewport, Log, All);

void UAuxiliaryGameViewportClient::AddAssociation(FViewport& InViewport)
{
	Super::AddAssociation(InViewport);

	UGameInstance* OwningGameInstance = GetGameInstance();
	if (!OwningGameInstance)
	{
		UE_LOGF(LogAuxiliaryGameViewport, Warning, "Game Viewport Client has no Game Instance.  The viewport will not be able to draw anything.");
		return;
	}

	// Restore the WorldContext->GameViewport back-pointer if UEngine::CleanupGameViewport cleared it
	// during a previous detach. SetupInitialLocalPlayer below uses this back-pointer to find the WorldContext.
	if (FWorldContext* WorldContext = OwningGameInstance->GetWorldContext())
	{
		WorldContext->GameViewport = this;
	}

	FString Error;
	ULocalPlayer* LocalPlayer = OwningGameInstance->GetFirstGamePlayer();
	if (!LocalPlayer)
	{
		LocalPlayer = SetupInitialLocalPlayer(Error);
	}
	if (!LocalPlayer)
	{
		UE_LOGF(LogAuxiliaryGameViewport, Error, "Failed to create LocalPlayer for world: %ls", *Error);
		return;
	}

	if (!LocalPlayer->SpawnPlayActor(OwningGameInstance->GetWorldContext()->LastURL.ToString(), Error, OwningGameInstance->GetWorld()))
	{
		UE_LOGF(LogAuxiliaryGameViewport, Error, "Couldn't spawn player actors: %ls", *Error);
	}
}
