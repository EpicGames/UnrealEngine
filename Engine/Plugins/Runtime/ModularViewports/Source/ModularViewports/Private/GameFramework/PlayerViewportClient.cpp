// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/PlayerViewportClient.h"
#include "ViewportFunctions.h"
#include "CanvasTypes.h"
#include "InputKeyEventArgs.h"
#include "UnrealClient.h"
#include "SceneView.h"
#include "Engine/World.h"
#include "Engine/LocalPlayer.h"
#include "Engine/GameViewportClient.h"
#include "GameFramework/InputSettings.h"
#include "GameFramework/PlayerController.h"
#include "Camera/PlayerCameraManager.h"
#include "Engine/GameInstance.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PlayerViewportClient)

namespace UE::Private::PlayerViewportClient
{
	namespace
	{
		thread_local bool bConstructingViaCreate = false;

		TUniquePtr<FSceneViewFamilyContext> BuildViewFamily(FViewport& Viewport, ULocalPlayer& Player, UWorld& World)
		{
			using namespace UE::Engine;

			TUniquePtr<FSceneViewFamilyContext> ViewFamily = MakeGameplayViewFamily(Viewport, World);

			FVector ViewLocation;
			FRotator ViewRotation;
			FSceneView* View = Player.CalcSceneView(ViewFamily.Get(), ViewLocation, ViewRotation, &Viewport);

			if (!View)
			{
				return nullptr;
			}

			Player.LastViewLocation = ViewLocation;

			return ViewFamily;
		}

		/** Returns true if DeviceId is assigned to a platform user different from Player's.
		 * Returns false if the device is unassigned.
		 */
		bool IsDeviceFromDifferentUser(const ULocalPlayer& Player, FInputDeviceId DeviceId)
		{
			IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();
			const FPlatformUserId DeviceUser = DeviceMapper.GetUserForInputDevice(DeviceId);
			if (!DeviceUser.IsValid())
			{
				return false; // Device not assigned to any player, accept it.
			}

			return DeviceUser != Player.GetPlatformUserId();
		}
	}
}

namespace UE::Engine
{
	using namespace UE::Private::PlayerViewportClient;

	UPlayerViewportClient::UPlayerViewportClient(const FObjectInitializer& ObjectInitializer)
		: Super(ObjectInitializer)
	{
		if (!HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
		{
			ensureMsgf(bConstructingViaCreate,
				TEXT("UPlayerViewportClient must be constructed via UPlayerViewportClient::Create()."));
		}
	}

	UPlayerViewportClient* UPlayerViewportClient::Create(ULocalPlayer& InPlayer, UObject* Outer)
	{
		if (UScriptViewportClient* const Existing = InPlayer.GetViewportClient();
			!ensureMsgf(Existing == InPlayer.ViewportClient.Get(),
				TEXT("ULocalPlayer %s already has an exclusive viewport client %s; PlayerInput requires a 1:1 viewport:player relationship."),
				*InPlayer.GetName(), *Existing->GetName()))
		{
			return nullptr;
		}

		UPlayerViewportClient* PlayerClient;
		{
			TGuardValue ConstructionGuard(bConstructingViaCreate, true);
			PlayerClient = Outer
				? NewObject<UPlayerViewportClient>(Outer)
				: NewObject<UPlayerViewportClient>();
		}

		PlayerClient->Player = &InPlayer;

		InPlayer.Origin = FVector2D::ZeroVector;
		InPlayer.Size = FVector2D(1.0, 1.0);

		InPlayer.OverrideViewportClient(PlayerClient);

		return PlayerClient;
	}

	void UPlayerViewportClient::BeginDestroy()
	{
		if (ULocalPlayer* const PlayerPtr = Player.Get())
		{
			if (PlayerPtr->GetViewportClient() == this)
			{
				PlayerPtr->OverrideViewportClient(nullptr);
			}
		}

		Super::BeginDestroy();
	}

	UWorld* UPlayerViewportClient::GetWorld() const
	{
		ULocalPlayer* PlayerPtr = Player.Get();
		if (PlayerPtr)
		{
			APlayerController* Controller = PlayerPtr->PlayerController;
			if (IsValid(Controller))
			{
				return Controller->GetWorld();
			}
			return PlayerPtr->ViewportClient ? PlayerPtr->ViewportClient->GetWorld() : nullptr;
		}
		return nullptr;
	}

	void UPlayerViewportClient::Draw(FViewport* Viewport, FCanvas* Canvas)
	{
		check(Viewport);
		check(Canvas);

		ULocalPlayer* PlayerPtr = Player.Get();
		if (!PlayerPtr)
		{
			return;
		}
		APlayerController* Controller = PlayerPtr->PlayerController;
		if (!Controller)
		{
			return;
		}
		UWorld* World = Controller->GetWorld();
		if (!World)
		{
			return;
		}

		if (const TUniquePtr<FSceneViewFamilyContext> ViewFamily = BuildViewFamily(*Viewport, *PlayerPtr, *World))
		{
			for (const FSceneView* const View : ViewFamily->Views)
			{
				AddStreamingViewInfo(*World, *View);
			}
			World->LastRenderTime = World->GetTimeSeconds();
			TryRenderViewFamily(*ViewFamily, *Canvas);
		}

		// Propagate camera cut to the viewport for temporal effects (TAA, motion vectors).
		if (APlayerCameraManager* CameraManager = Controller->PlayerCameraManager)
		{
			Viewport->SetCameraCut(CameraManager->bGameCameraCutThisFrame);
			CameraManager->bGameCameraCutThisFrame = false;
		}
	}

	EMouseCursor::Type UPlayerViewportClient::GetCursor(FViewport* Viewport, int32 X, int32 Y)
	{
		if (Player.IsValid() && Player->PlayerController)
		{
			return Player->PlayerController->GetMouseCursor();
		}
		return FViewportClient::GetCursor(Viewport, X, Y);
	}

	bool UPlayerViewportClient::RequiresUncapturedAxisInput() const
	{
		if (Player.IsValid() && Player->PlayerController)
		{
			return Player->PlayerController->ShouldShowMouseCursor();
		}
		return false;
	}

	void UPlayerViewportClient::LostFocus(FViewport* Viewport)
	{
		if (!Player.IsValid() || !Player->PlayerController)
		{
			return;
		}

		const bool bShouldFlush = GetDefault<UInputSettings>()->bShouldFlushPressedKeysOnViewportFocusLost
			|| Player->PlayerController->ShouldFlushKeysWhenViewportFocusChanges();
		if (bShouldFlush)
		{
			Player->PlayerController->FlushPressedKeys();
		}
	}

	bool UPlayerViewportClient::InputKey(const FInputKeyEventArgs& EventArgs)
	{
		// Only filter gamepad keys. Mouse and keyboard can't be associated with one player or another by device ID.  If this is desired, you should
		// re-route input to the viewport you want at the Slate level of abstraction, or to the Controller you want by subclassing this Viewport
		// Client and overriding this function.
		if (EventArgs.IsGamepad() && bFilterInputByPlayer && Player.IsValid() && IsDeviceFromDifferentUser(*Player, EventArgs.InputDevice))
		{
			return false;
		}

		if (Player.IsValid() && Player->PlayerController)
		{
			const bool bResult = Player->PlayerController->InputKey(EventArgs);

			// Mouse buttons are always considered consumed by the viewport.
			if (!bResult && EventArgs.Key.IsMouseButton())
			{
				return true;
			}
			return bResult;
		}
		return false;
	}

	bool UPlayerViewportClient::InputAxis(const FInputKeyEventArgs& Args)
	{
		// Only filter gamepad axes. Mouse and keyboard axes can't be associated with one player or another by device ID.  If this is desired, you
		// should re-route input to the viewport you want at the Slate level of abstraction, or to the Controller you want by subclassing this
		// Viewport Client and overriding this function.
		if (Args.IsGamepad() && bFilterInputByPlayer && Player.IsValid() && IsDeviceFromDifferentUser(*Player, Args.InputDevice))
		{
			return false;
		}

		if (Player.IsValid() && Player->PlayerController)
		{
			return Player->PlayerController->InputKey(Args);
		}
		return false;
	}

	bool UPlayerViewportClient::InputTouch(
		FViewport* const InViewport,
		const FTouchId TouchId,
		const ETouchType::Type Type,
		const FVector2D& TouchLocation,
		const float Force,
		const uint64 Timestamp)
	{
		// Don't consult device ID, as we shouldn't have device A associated with Player A sending touches to viewport B in the first place (touching
		// sets focus, determines receiver at Slate Application level).
		if (Player.IsValid() && Player->PlayerController)
		{
			return Player->PlayerController->InputTouch(TouchId, Type, TouchLocation, Force, Timestamp);
		}
		return false;
	}

	bool UPlayerViewportClient::InputMotion(
		FViewport* InViewport,
		const FInputDeviceId DeviceId,
		const FVector& Tilt,
		const FVector& RotationRate,
		const FVector& Gravity,
		const FVector& Acceleration,
		const uint64 Timestamp)
	{
		if (bFilterInputByPlayer && Player.IsValid() && IsDeviceFromDifferentUser(*Player, DeviceId))
		{
			return false;
		}

		if (Player.IsValid() && Player->PlayerController)
		{
			return Player->PlayerController->InputMotion(DeviceId, Tilt, RotationRate, Gravity, Acceleration, Timestamp);
		}
		return false;
	}

	bool UPlayerViewportClient::InputGesture(
		FViewport* InViewport,
		const FInputDeviceId DeviceId,
		EGestureEvent GestureType,
		EGesturePhase GesturePhase,
		const FVector2D& GestureDelta,
		bool bIsDirectionInvertedFromDevice,
		const uint64 Timestamp)
	{
		// Don't consult device ID, as we shouldn't have device A associated with Player A sending touches to viewport B in the first place (touching
		// sets focus, determines receiver at Slate Application level).
		if (Player.IsValid() && Player->PlayerController)
		{
			return Player->PlayerController->InputGesture(DeviceId, GestureType, GesturePhase, GestureDelta, bIsDirectionInvertedFromDevice, Timestamp);
		}
		return false;
	}
} // namespace UE::Engine
