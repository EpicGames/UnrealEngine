// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportFunctions.h"
#include "CanvasTypes.h"
#include "Engine/Engine.h"
#include "EngineModule.h"
#include "RendererInterface.h"
#include "LegacyScreenPercentageDriver.h"
#include "SceneView.h"
#include "ShowFlags.h"
#include "Slate/SceneViewport.h"
#include "UnrealClient.h"
#include "ViewportClient.h"
#include "Engine/World.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/SViewport.h"
#include "Widgets/SWindow.h"

namespace UE::Engine
{
	TUniquePtr<FSceneViewFamilyContext> MakeGameplayViewFamily(const FViewport& Viewport, const UWorld& World)
	{
		TUniquePtr<FSceneViewFamilyContext> ViewFamily = MakeUnique<FSceneViewFamilyContext>(
			FSceneViewFamily::ConstructionValues(&Viewport, World.Scene, ESFIM_Game)
				.SetRealtimeUpdate(true)
		);
		EngineShowFlagOverride(ESFIM_Game, ViewFamily->ViewMode, ViewFamily->EngineShowFlags, false);

		ViewFamily->bIsHDR = Viewport.IsHDRViewport();

		if (ViewFamily->GetScreenPercentageInterface() == nullptr)
		{
			ViewFamily->SetScreenPercentageInterface(new FLegacyScreenPercentageDriver(*ViewFamily, 1.0));
		}

		return ViewFamily;
	}

	bool TryRenderViewFamily(FSceneViewFamilyContext& ViewFamily, FCanvas& Canvas)
	{
		if (FSlateApplication::Get().GetPlatformApplication()->IsAllowedToRender())
		{
			GetRendererModule().BeginRenderingViewFamily(&Canvas, &ViewFamily);
			Canvas.Flush_GameThread();
			return true;
		}
		return false;
	}

	SWindow::FArguments ExactPositionWindow(const FVector2f& Position, SWindow::FArguments&& Arguments)
	{
		return MoveTemp(Arguments)
			.AutoCenter(EAutoCenter::None)
			.SaneWindowPlacement(false) // "Sane" window placement can force AutoCenter to become PreferredWorkArea depending on the position
			.ScreenPosition(Position);
	}

	SWindow::FArguments FixedSizeWindow(FVector2f Size, SWindow::FArguments&& Arguments)
	{
		return MoveTemp(Arguments)
			.ClientSize(Size)
			.SaneWindowPlacement(false) // "Sane" window placement can clamp the window's size even if the sizing rule is FixedSize
			.SizingRule(ESizingRule::FixedSize);
	}

	SWindow::FArguments NoControlsWindow(SWindow::FArguments&& Arguments)
	{
		return MoveTemp(Arguments)
			.SupportsMinimize(false)
			.SupportsMaximize(false)
			.HasCloseButton(false);
	}

	TSharedRef<FSceneViewport> SetupRendering(const TStrongPtrVariant<FViewportClient>& Client, const TSharedRef<SViewport>& ViewportWidget)
	{
		check(IsInGameThread());

		TSharedRef SceneViewport = FSceneViewport::Create(Client, ViewportWidget);

		if (FSlateApplication::IsInitialized())
		{
			if (TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(ViewportWidget))
			{
				if (Window->GetViewport() == nullptr)
				{
					Window->SetViewport(SceneViewport);
				}
			}
		}

		if (GEngine)
		{
			GEngine->RegisterViewport(StaticCastSharedRef<FViewport>(SceneViewport));
		}

		return SceneViewport;
	}
}
