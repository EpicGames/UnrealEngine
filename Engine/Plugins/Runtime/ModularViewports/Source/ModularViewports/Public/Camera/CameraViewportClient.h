// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/PrimitiveComponent.h"
#include "GameplayViewportClient.h"
#include "InputCoreTypes.h"
#include "UObject/GCObject.h"
#include "SceneTypes.h"

#define UE_API MODULARVIEWPORTS_API

class UCameraComponent;
class FSceneViewFamilyContext;

namespace UE
{
	/** A Viewport Client that renders a single UCameraComponent. */
	class FCameraViewportClient : public FGameplayViewportClient, public FGCObject
	{
		TWeakObjectPtr<const UCameraComponent> Camera;
		TWeakObjectPtr<UWorld> ViewStateWorld;
		FSceneViewStateReference ViewState;
		ETouchEventFamilies EnabledTouchEventFamilies;
		ECollisionChannel TouchTraceChannel;
		float TouchTraceDistance;
		TWeakObjectPtr<UPrimitiveComponent> TouchedPrimitives[EKeys::NUM_TOUCH_KEYS];

	public:
		UE_API FCameraViewportClient(
			const UCameraComponent&,
			ETouchEventFamilies EnabledTouchEventFamilies = ETouchEventFamilies::All,
			ECollisionChannel TouchTraceChannel = ECC_Visibility,
			float TouchTraceDistance = 100000.0f);

		ECollisionChannel GetTouchTraceChannel() const
		{
			return TouchTraceChannel;
		}

		float GetTouchTraceDistance() const
		{
			return TouchTraceDistance;
		}

		const TWeakObjectPtr<const UCameraComponent>& GetCameraComponent() const
		{
			return Camera;
		}

		/** Extracts projection data (view rect, rotation matrix, projection matrix) from the camera.
		 *
		 * Only safe to call within GC Lock.
		 */
		UE_API void CalculateProjectionData(const FViewport& Viewport, FSceneViewProjectionData& OutProjectionData) const;

		// FGCObject interface
		UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
		UE_API virtual FString GetReferencerName() const override;
		// ~FGCObject interface

		// FViewportClient interface
		UE_API virtual UWorld* GetWorld() const override;
		UE_API virtual void Draw(FViewport* Viewport, FCanvas* Canvas) override;
		UE_API virtual bool InputTouch(FViewport* const Viewport, const FTouchId TouchId, const ETouchType::Type Type,
			const FVector2D& TouchLocation, const float Force, const uint64 Timestamp) override;
		// ~FViewportClient interface

	protected:
		void SetTouchTraceChannel(ECollisionChannel InTouchTraceChannel)
		{
			TouchTraceChannel = InTouchTraceChannel;
		}

		void SetTouchTraceDistance(float InTouchTraceDistance)
		{
			TouchTraceDistance = InTouchTraceDistance;
		}
	};
}

#undef UE_API
