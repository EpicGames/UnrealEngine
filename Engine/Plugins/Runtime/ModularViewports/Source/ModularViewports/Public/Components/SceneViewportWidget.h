// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "Components/ContentWidget.h"
#include "Engine/AuxiliaryGameInstance.h"
#include "Templates/PointerVariants.h"
#include "SceneViewportWidget.generated.h"

#define UE_API MODULARVIEWPORTS_API

class FSceneViewport;
class FViewportClient;
class SViewport;
class UCameraComponent;
class UGameInstance;
class ULocalPlayer;

/** UMG wrapper around SViewport + FSceneViewport, which allows you to compose Viewport Clients (e.g. Game Viewport Client) as Slate Widgets.
 * 
 * Provides properties/functions for setting up with a GameInstance, Camera, or Player.  
 */
UCLASS(MinimalAPI)
class USceneViewport final : public UWidget
{
	GENERATED_BODY()

private:
	/**
	 * This world will be automatically spawned and bound to this widget upon initialization.  If null, the viewport will be blank until a "Show..."
	 * function is called.
	 */
	UPROPERTY(EditAnywhere, Category = "Viewport")
	TSoftObjectPtr<UWorld> AutoLoadWorldAsset;

	/** The Viewport Client will draw directly to the SWindow's backbuffer, skipping Slate composition.
	 * 
	 * Improves performance for real-time clients, but does not allow Slate-level effects to be applied to this widget, and may cause wrong z-order.
	 * If this is true and the client early-exits from Draw, the previously rendered frame will not persist. 
	 */
	UPROPERTY(EditAnywhere, Category = "Viewport")
	uint8 bRenderDirectlyToWindow : 1;

	/** Whether or not to enable gamma correction. Doesn't apply when rendering directly to a backbuffer. */
	UPROPERTY(EditAnywhere, Category = "Viewport", meta = (EditCondition = "!bRenderDirectlyToWindow"))
	uint8 bEnableGammaCorrection : 1;

	/** Whether or not to reverse the gamma correction done to the texture in this viewport.  Ignores the bEnableGammaCorrection setting */
	UPROPERTY(EditAnywhere, Category = "Viewport", meta = (EditCondition = "!bRenderDirectlyToWindow"))
	uint8 bReverseGammaCorrection : 1;

	/** Allow this viewport to blend with its background. */
	UPROPERTY(EditAnywhere, Category = "Viewport", meta = (EditCondition = "!bRenderDirectlyToWindow"))
	uint8 bEnableBlending : 1;

	/** True if the viewport texture has pre-multiplied alpha */
	UPROPERTY(EditAnywhere, Category = "Viewport", meta = (EditCondition = "!bRenderDirectlyToWindow"))
	uint8 bPreMultipliedAlpha : 1;

	/**
	 * If true, the viewport's texture alpha is ignored when performing blending.  In this case only the viewport tint opacity is used
	 * If false, the texture alpha is used during blending
	 */
	UPROPERTY(EditAnywhere, Category = "Viewport", meta = (EditCondition = "!bRenderDirectlyToWindow"))
	uint8 bIgnoreTextureAlpha : 1;

	/** The primary SWidget that this UWidget represents. */
	TSharedPtr<SViewport> ViewportWidget;

	/** The currently bound viewport client. FSceneViewport also holds this strongly once Bind() runs, so this field's job is
	 * to carry the binding intent between BindGame/BindCamera and a later Bind() (Bind can be deferred when the viewport
	 * widget hasn't been built yet). By contract, a UObject-backed variant here is always a UGameViewportClient.
	 */
	TStrongPtrVariant<FViewportClient> Client;

	/** Stores a Game Instance that is created from AutoLoadWorldAsset (not relevant for ShowGame). */
	TUniquePtr<UE::Engine::FAuxiliaryGameInstance> GameInstanceAssembly;

	TSharedPtr<FSceneViewport> SceneViewport;


	USceneViewport(const FObjectInitializer&);
public:
	/** If the Auto Load World Asset property of this widget was specified, returns the Game Instance which was spawned as a result. */
	UFUNCTION(BlueprintPure, Category = "MultiWorld")
	UE_API UGameInstance* GetInnerGameInstance() const;

	UE_API void ShowGame(const TSoftObjectPtr<UWorld>& Asset);

	UE_API void ShowCamera(const UCameraComponent&);

	UE_API void ShowPlayer(ULocalPlayer&);

	UE_API void Unbind();

private:
	// UObject
	UE_API virtual void BeginDestroy() override;
	// ~UObject
	
	// UVisual
	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	// ~UVisual

	// UWidget
#if WITH_EDITOR
	UE_API virtual const FText GetPaletteCategory() override;
#endif
	UE_API virtual TSharedRef<SWidget> RebuildWidget() override;
	// ~UWidget

	void Bind();
};

#undef UE_API
