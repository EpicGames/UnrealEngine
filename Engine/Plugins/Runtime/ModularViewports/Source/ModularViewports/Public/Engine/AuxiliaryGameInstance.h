// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/World.h"
#include "Engine/GameInstance.h"

#define UE_API MODULARVIEWPORTS_API

namespace UE::Engine
{
	/**
	 * Instantiates a given World within a new Game Instance.  Destroying this object will shut down and destroy both.
	 * Works across both PIE and non-editor.
	 * Use for spatially discontiguous picture-in-picture, second window content, or other multi-tenant scenarios.
	 *
	 * Produces a "headless" world.  To create players, you must first associate it with a viewport-client.  To spawn player-controllers, you must
	 * first associate it with a viewport.
	 *
	 * Note: BeginPlay is called immediately, *before* any players exist.  This is different from the primary game instance's standalone behavior (but
	 * will be familiar to anyone who has developed multiplayer games).  Actors in the world will find that the Player Controller doesn't exist yet on
	 * Begin Play.  Getting the Player Controller on BeginPlay is an anti-practice anyhow.  If you put input-binding code in the Player Controller
	 * (where it belongs) and use proper abstractions + event-driven orchestration, this will not be a problem.
	 *
	 * Does NOT support:
	 * - In-editor but NOT from-PIE (i.e. from custom editor tools)
	 * - Networking/replication
	 * - Travel
	 * - Multiple instances of the same world asset
	 */
	class FAuxiliaryGameInstance final : public FNoncopyable
	{
		TStrongObjectPtr<UGameInstance> GameInstance;

#if WITH_EDITOR
		FDelegateHandle EndPIEHandle;
#endif

	public:
		static UE_API TUniquePtr<FAuxiliaryGameInstance> MakeUnique(const TSoftObjectPtr<UWorld>& Asset);
		static UE_API TSharedPtr<FAuxiliaryGameInstance> MakeShared(const TSoftObjectPtr<UWorld>& Asset);
	private:
		static FAuxiliaryGameInstance* Make(const TSoftObjectPtr<UWorld>& Asset);

		explicit FAuxiliaryGameInstance(UGameInstance&);

		/**
		 * Tear down the World, Game Instance, and World Context that this object owns.
		 * 
		 * Idempotent — a second call returns immediately.  Callable from either the destructor or the FEditorDelegates::EndPIE handler.
		 */
		void Teardown();
	public:
		UE_API ~FAuxiliaryGameInstance();

		UGameInstance* GetGameInstance() const
		{
			return GameInstance.Get();
		}

		UWorld* GetWorld() const
		{
			return GameInstance->GetWorld();
		}
	};
}

#undef UE_API
