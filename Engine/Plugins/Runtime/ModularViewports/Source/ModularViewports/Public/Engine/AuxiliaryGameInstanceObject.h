// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "Engine/AuxiliaryGameInstance.h"
#include "AuxiliaryGameInstanceObject.generated.h"

#define UE_API MODULARVIEWPORTS_API

class UGameInstance;
class UWorld;

/**
 * UObject wrapper around UE::Engine::FAuxiliaryGameInstance.
 *
 * Lets Blueprint and other UFUNCTION-only APIs hold an FAuxiliaryGameInstance via a regular UObject reference; the wrapped FAuxiliaryGameInstance is
 * destroyed when this UObject is garbage-collected.
 */
UCLASS(MinimalAPI, BlueprintType)
class UAuxiliaryGameInstance : public UObject
{
	GENERATED_BODY()

	TUniquePtr<UE::Engine::FAuxiliaryGameInstance> Inner;

public:
	/** Creates a new wrapper that owns a fresh FAuxiliaryGameInstance for the given world asset. */
	static UE_API UAuxiliaryGameInstance* Make(const TSoftObjectPtr<UWorld>& Asset);

	UE::Engine::FAuxiliaryGameInstance* Get() const
	{
		return Inner.Get();
	}

	UFUNCTION(BlueprintPure, Category = "AuxiliaryGameInstance")
	UE_API UGameInstance* GetGameInstance() const;

	UE_API virtual UWorld* GetWorld() const override;
	UE_API virtual void BeginDestroy() override;
};

#undef UE_API
