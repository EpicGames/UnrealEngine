// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/GameViewportClient.h"
#include "AuxiliaryGameViewportClient.generated.h"

#define UE_API MODULARVIEWPORTS_API

/**
 * Game viewport client used by FAuxiliaryGameInstance.
 *
 * Spawns player actor in response to viewport being attached.
 */
UCLASS(MinimalAPI)
class UAuxiliaryGameViewportClient : public UGameViewportClient
{
	GENERATED_BODY()

public:
	UE_API virtual void AddAssociation(FViewport& InViewport) override;
};

#undef UE_API
