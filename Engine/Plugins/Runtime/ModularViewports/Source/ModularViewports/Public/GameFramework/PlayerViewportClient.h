// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/ScriptViewportClient.h"
#include "PlayerViewportClient.generated.h"

class ULocalPlayer;
class FSceneViewFamilyContext;

#define UE_API MODULARVIEWPORTS_API

namespace UE::Engine
{
	/** A Viewport Client that renders a single ULocalPlayer using their camera manager.
	 *
	 * Halfway between FCameraViewportClient (raw camera, no input) and UGameViewportClient (all players, split-screen).
	 * Routes input to the player's controller.
	 *
	 * Must be constructed via the static Create() factory — direct NewObject<> construction trips an ensure.
	 */
	UCLASS(transient, MinimalAPI)
	class UPlayerViewportClient : public UScriptViewportClient
	{
		GENERATED_BODY()

		TWeakObjectPtr<ULocalPlayer> Player;
		bool bFilterInputByPlayer = false;

	public:
		UE_API UPlayerViewportClient(const FObjectInitializer& ObjectInitializer);

		/**
		 * Creates a UPlayerViewportClient bound to Player.  Returns nullptr if Player is already associated with a non-GameViewportClient viewport
		 * client.
		 */
		static UE_API UPlayerViewportClient* Create(ULocalPlayer& Player, UObject* Outer = nullptr);

		const TWeakObjectPtr<ULocalPlayer>& GetLocalPlayer() const
		{
			return Player;
		}

		/** When true, gamepad input from devices assigned to other players is ignored.
		 * Mouse, keyboard, and touch input always passes through regardless of this setting.
		 */
		void SetFilterInputByPlayer(bool bFilter)
		{
			bFilterInputByPlayer = bFilter;
		}

		bool GetFilterInputByPlayer() const
		{
			return bFilterInputByPlayer;
		}

		// UObject interface
		UE_API virtual void BeginDestroy() override;
		// ~UObject interface

		// FViewportClient interface
		UE_API virtual UWorld* GetWorld() const override;
		UE_API virtual void Draw(FViewport* Viewport, FCanvas* Canvas) override;
		UE_API virtual EMouseCursor::Type GetCursor(FViewport* Viewport, int32 X, int32 Y) override;
		UE_API virtual bool RequiresUncapturedAxisInput() const override;
		UE_API virtual void LostFocus(FViewport* Viewport) override;
		UE_API virtual bool InputKey(const FInputKeyEventArgs& EventArgs) override;
		UE_API virtual bool InputAxis(const FInputKeyEventArgs& Args) override;
		UE_API virtual bool InputTouch(FViewport* const Viewport, const FTouchId TouchId, const ETouchType::Type Type, const FVector2D& TouchLocation, const float Force, const uint64 Timestamp) override;
		UE_API virtual bool InputMotion(FViewport* Viewport, const FInputDeviceId DeviceId, const FVector& Tilt, const FVector& RotationRate, const FVector& Gravity, const FVector& Acceleration, const uint64 Timestamp) override;
		UE_API virtual bool InputGesture(FViewport* Viewport, const FInputDeviceId DeviceId, EGestureEvent GestureType, EGesturePhase GesturePhase, const FVector2D& GestureDelta, bool bIsDirectionInvertedFromDevice, const uint64 Timestamp) override;
		// ~FViewportClient interface
	};
}

#undef UE_API
