// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"
#include "Templates/SubclassOf.h"
#include "ModularViewportsSettings.generated.h"

#define UE_API MODULARVIEWPORTS_API

class UAuxiliaryGameViewportClient;

/** Project settings for the ModularViewports plugin. */
UCLASS(MinimalAPI, config = Engine, defaultconfig, meta = (DisplayName = "Modular Viewports"))
class UModularViewportsSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	/** Default viewport-client class instantiated for each FAuxiliaryGameInstance.  Performs player setup in its Bind override. */
	UPROPERTY(config, EditAnywhere, Category = "Modular Viewports", meta = (MetaClass = "/Script/ModularViewports.AuxiliaryGameViewportClient"))
	TSubclassOf<UAuxiliaryGameViewportClient> AuxiliaryGameInstanceClass;
};

#undef UE_API
