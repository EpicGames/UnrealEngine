// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Templates/PointerVariants.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "Widgets/SWindow.h"

class FCanvas;
class FSceneViewFamilyContext;
class FSceneViewport;
class FViewport;
class FViewportClient;
class SViewport;
class UWorld;

#define UE_API MODULARVIEWPORTS_API

namespace UE::Engine
{
	/** Creates a view family with typical defaults for rendering Game Instance content.
	 *
	 * Sets up a realtime FSceneViewFamilyContext with engine show flag overrides, HDR viewport state, and a default screen percentage driver.
	 */
	UE_API TUniquePtr<FSceneViewFamilyContext> MakeGameplayViewFamily(const FViewport& Viewport, const UWorld& World);

	/** Renders the given view family to the given canvas using the renderer module (if allowed). */
	UE_API bool TryRenderViewFamily(FSceneViewFamilyContext& ViewFamily, FCanvas& Canvas);

	/** Produces SWindow parameters that ensure a specific placement.
	 * 
	 * Use it like: `SArgumentsNew(ExactLocationWindow(FVector2f(100, 100)), SWindow).OtherParam(...)[ Child ]`
	 */
	UE_API SWindow::FArguments ExactPositionWindow(const FVector2f& Position, SWindow::FArguments&& Arguments = SWindow::FArguments());

	/** Produces SWindow parameters ideal for a fixed-size window.
	 * 
	 * Use it like: `SArgumentsNew(FixedSizeWindow(FVector2f(800, 600)), SWindow).OtherParam(...)[ Child ]`
	 */
	UE_API SWindow::FArguments FixedSizeWindow(FVector2f Size, SWindow::FArguments&& Arguments = SWindow::FArguments());

	/** Produces an SWindow with no exit/minimize/maximize buttons
	 * 
	 * Use it like: `SArgumentsNew(NoControlsWindow(), SWindow).OtherParam(...)[ Child ]`
	 */
	UE_API SWindow::FArguments NoControlsWindow(SWindow::FArguments&& Arguments = SWindow::FArguments());

	/** Sets up the FViewportClient to render into the SViewport by creating a FSceneViewport, and performing the typical boilerplate setup steps.
	 *
	 * - Matches the FSceneViewport's size to that of the given SViewport.
	 * - Makes this the SViewport's parent window's primary viewport, unless it does not yet belong to a window, or the window's primary viewport is
	 *   already set.
	 * - Registers the FViewport with GEngine to be drawn every frame.
	 */
	UE_API TSharedRef<FSceneViewport> SetupRendering(const TStrongPtrVariant<FViewportClient>& Client, const TSharedRef<SViewport>& ViewportWidget);
}

#undef UE_API
